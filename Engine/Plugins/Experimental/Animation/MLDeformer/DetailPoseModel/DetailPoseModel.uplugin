{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "ML Deformer Detail Pose Model",
	"Description": "Detail Pose Model for the ML Deformer Framework",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/5.0/en-US/using-the-machine-learning-deformer-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DetailPoseModel",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DetailPoseModelEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "NeuralMorphModel",
			"Enabled": true
		}
	]
}
