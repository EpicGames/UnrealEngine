// Copyright Epic Games, Inc. All Rights Reserved.
#include "DetailPoseModel.h"
#include "DetailPoseModelVizSettings.h"
#include "DetailPoseModelInstance.h"
#include "DetailPoseModelInputInfo.h"
#include "MLDeformerComponent.h"
#include "MLDeformerAsset.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DetailPoseModel)

#define LOCTEXT_NAMESPACE "DetailPoseModel"

// Implement our module.
namespace UE::DetailPoseModel
{
	class DETAILPOSEMODEL_API FDetailPoseModelModule
		: public IModuleInterface
	{
	};
}
IMPLEMENT_MODULE(UE::DetailPoseModel::FDetailPoseModelModule, DetailPoseModel)

// Our log category for this model.
DETAILPOSEMODEL_API DEFINE_LOG_CATEGORY(LogDetailPoseModel)

//////////////////////////////////////////////////////////////////////////////

UDetailPoseModel::UDetailPoseModel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// We only want to use global mode in this model.
	Mode = ENeuralMorphMode::Global;

	// Create the visualization settings for this model.
	// Never directly create one of the frameworks base classes such as the FMLDeformerMorphModelVizSettings as
	// that can cause issues with detail customizations.
#if WITH_EDITORONLY_DATA
	SetVizSettings(ObjectInitializer.CreateEditorOnlyDefaultSubobject<UDetailPoseModelVizSettings>(this, TEXT("DetailPoseModelVizSettings")));
#endif
}

UMLDeformerModelInstance* UDetailPoseModel::CreateModelInstance(UMLDeformerComponent* Component)
{
	return NewObject<UDetailPoseModelInstance>(Component);
}

UMLDeformerInputInfo* UDetailPoseModel::CreateInputInfo()
{
	return NewObject<UDetailPoseModelInputInfo>(this, NAME_None, RF_Transactional);
}

#if WITH_EDITORONLY_DATA
	UAnimSequence* UDetailPoseModel::GetDetailPosesAnimSequence() const
	{ 
		return DetailPosesAnimSequence.LoadSynchronous();
	}

	UGeometryCache* UDetailPoseModel::GetDetailPosesGeomCache() const
	{ 
		return DetailPosesGeomCache.LoadSynchronous();
	}
#endif

void UDetailPoseModel::Serialize(FArchive& Archive)
{
	// If we are saving and have raw deltas, strip the detail pose deltas from them, as we can just generate them when needed.
	// This can keep the editor asset (non cooked) as small as possible.
	TArray<FVector3f> DeltasBackup;
	if (Archive.IsSaving() && GetInputInfo() && !GetMorphTargetDeltas().IsEmpty())
	{
		const UDetailPoseModelInputInfo* DetailPoseModelInputInfo = Cast<UDetailPoseModelInputInfo>(GetInputInfo());
		check(DetailPoseModelInputInfo);
		const int32 NumVertices = DetailPoseModelInputInfo->GetNumBaseMeshVertices();
		if (NumVertices > 0)
		{
			const int32 NumMorphTargets = DetailPoseModelInputInfo->GetNumGlobalMorphTargets() + 1; // +1 for the means.
			const TConstArrayView<FVector3f> MorphDeltasWithoutDetailPoses(GetMorphTargetDeltas().GetData(), NumVertices * NumMorphTargets);
			TArray<FVector3f> StrippedDeltas(MorphDeltasWithoutDetailPoses);
			DeltasBackup = GetMorphTargetDeltas();
			SetMorphTargetDeltas(StrippedDeltas);
		}
	}

	Super::Serialize(Archive);

	// Recover the original deltas we had before we stripped out the detail pose deltas.
	if (!DeltasBackup.IsEmpty())
	{
		SetMorphTargetDeltas(DeltasBackup);
	}
}

void UDetailPoseModel::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	#if WITH_EDITORONLY_DATA
		const FString DetailPosesAnimSequenceString = DetailPosesAnimSequence.ToSoftObjectPath().ToString();
		const FString DetailPosesGeomCacheString = DetailPosesGeomCache.ToSoftObjectPath().ToString();
		Context.AddTag(FAssetRegistryTag("MLDeformer.DetailPoseModel.DetailPosesAnimSequence", DetailPosesAnimSequenceString, FAssetRegistryTag::TT_Alphabetical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.DetailPoseModel.DetailPosesGeometryCache", DetailPosesGeomCacheString, FAssetRegistryTag::TT_Alphabetical));		
	#endif

	const UDetailPoseModelInputInfo* DetailPoseModelInputInfo = Cast<UDetailPoseModelInputInfo>(GetInputInfo());
	if (DetailPoseModelInputInfo)
	{
		Context.AddTag(FAssetRegistryTag("MLDeformer.DetailPoseModel.Trained.NumGlobalMorphTargets", FString::FromInt(DetailPoseModelInputInfo->GetNumGlobalMorphTargets()), FAssetRegistryTag::TT_Numerical));
	}

	Context.AddTag(FAssetRegistryTag("MLDeformer.DetailPoseModel.BlendSpeed", FString::Printf(TEXT("%f"), BlendSpeed), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.DetailPoseModel.RGBRange", FString::Printf(TEXT("%f"), RBFRange), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.DetailPoseModel.UseRBFInterpolation", bUseRBF ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
}

#undef LOCTEXT_NAMESPACE
