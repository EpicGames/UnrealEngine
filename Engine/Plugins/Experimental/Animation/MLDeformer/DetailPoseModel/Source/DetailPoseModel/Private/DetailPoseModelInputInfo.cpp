// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailPoseModelInputInfo.h"

void UDetailPoseModelInputInfo::Reset()
{
	Super::Reset();

	NumGlobalMorphTargets = 0;
}
