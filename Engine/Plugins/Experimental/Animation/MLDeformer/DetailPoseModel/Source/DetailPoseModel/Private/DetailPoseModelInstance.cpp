// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailPoseModelInstance.h"
#include "DetailPoseModel.h"
#include "DetailPoseModelVizSettings.h"
#include "DetailPoseModelInputInfo.h"
#include "NeuralMorphNetwork.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/ExternalMorphSet.h"
#include "Algo/MinElement.h"

#if INTEL_ISPC
	#include "DetailPoseModelPoseSearch.ispc.generated.h"
#endif

CSV_DECLARE_CATEGORY_MODULE_EXTERN(MLDEFORMERFRAMEWORK_API, MLDeformer);

namespace 
{
	float CalculateDetailPoseDistance(const TConstArrayView<float> ValuesA, const TConstArrayView<float> ValuesB)
	{
		check(ValuesA.Num() == ValuesB.Num());

		#if INTEL_ISPC
			return ispc::CalcDistance(ValuesA.GetData(), ValuesB.GetData(), ValuesA.Num());
		#else
			float Distance = 0.0f;
			const int32 NumValues = ValuesA.Num();
			for (int32 Index = 0; Index < NumValues; ++Index)
			{
				const float Diff = ValuesA[Index] - ValuesB[Index];
				Distance += Diff * Diff;
			}

			return Distance;
		#endif
	}

	void CalculateDetailPoseDistances(const UDetailPoseModel* DetailPoseModel, const TConstArrayView<float> CurrentPoseValues, TArray<float>& OutDetailPoseDistances)
	{
		const TArray<FDetailPoseModelDetailPose>& DetailPoses = DetailPoseModel->GetDetailPoses();
		for (int32 DetailPoseIndex = 0; DetailPoseIndex < DetailPoses.Num(); ++DetailPoseIndex)
		{
			const TConstArrayView<float> DetailPoseValues(DetailPoses[DetailPoseIndex].PoseValues);
			OutDetailPoseDistances[DetailPoseIndex] = CalculateDetailPoseDistance(DetailPoseValues, CurrentPoseValues);
		}
	}

	void UpdateWeight(float& OutWeight, float& InOutPrevWeight, float TargetWeight, float BlendSpeed)
	{
		const float BlendedWeight = BlendSpeed * TargetWeight + (1.0f - BlendSpeed) * InOutPrevWeight;
		OutWeight = BlendedWeight;
		InOutPrevWeight = BlendedWeight;
	}

	void UpdateRBFWeights(TArrayView<float> OutMorphWeights, TArrayView<float> SquaredDistances, TArrayView<float> InOutPrevWeights, float Sigma, float DetailPoseWeight, float BlendSpeed)
	{
		// NOTE: this method modifies the squared distances as well.

		const int32 NumDetailPoses = OutMorphWeights.Num();
		check(InOutPrevWeights.Num() == NumDetailPoses);
		check(SquaredDistances.Num() == NumDetailPoses);
		check(NumDetailPoses > 0);

		const float MinD2 = *Algo::MinElement(SquaredDistances);
		float SumWeights = 0.0f;
		const float Sigma2 = FMath::Max(Sigma * Sigma, 1e-6f);
		for (int32 Index = 0; Index < NumDetailPoses; ++Index)
		{
			float Weight = (SquaredDistances[Index] - MinD2) / Sigma2;
			constexpr float CutOff = 3.0f;
			if (Weight < CutOff)
			{
				Weight = FMath::Exp(-Weight);
				SquaredDistances[Index] = Weight;
				SumWeights += Weight;
			}
			else
			{
				SquaredDistances[Index] = 0.0f;
			}
		}

		SumWeights = FMath::Max(SumWeights, 1e-6f);
		for (int32 Index = 0; Index < NumDetailPoses; ++Index)
		{
			const float W = SquaredDistances[Index] / SumWeights * DetailPoseWeight;
			UpdateWeight(OutMorphWeights[Index], InOutPrevWeights[Index], W, BlendSpeed);
		}
	}

	void UpdateWeights(TArrayView<float> OutDetailPoseWeights, TConstArrayView<float> DistancePerDetailPose, TArrayView<float> InOutPrevDetailPoseWeights, float BlendSpeed, float ModelWeight)
	{
		// Find the detail pose index that has the smallest distance (to the current pose).
		const float* MinDistancePtr = Algo::MinElement(DistancePerDetailPose);
		const int32 MinDistanceDetailPoseIndex = MinDistancePtr - DistancePerDetailPose.GetData();

		for (int32 DetailPoseIndex = 0; DetailPoseIndex < OutDetailPoseWeights.Num(); ++DetailPoseIndex)
		{
			const float TargetWeight = (MinDistanceDetailPoseIndex == DetailPoseIndex) ? ModelWeight : 0.0f;
			UpdateWeight(OutDetailPoseWeights[DetailPoseIndex], InOutPrevDetailPoseWeights[DetailPoseIndex], TargetWeight, BlendSpeed); 
		}
	}

	void UpdateDetailPoseWeights(const UDetailPoseModel* DetailPoseModel, TArrayView<float> OutDetailPoseWeights, TArrayView<float> DistancePerDetailPose, TArrayView<float> InOutPrevDetailPoseWeights, float BlendSpeed, float ModelWeight)
	{
		#if WITH_EDITOR
			const UDetailPoseModelVizSettings* CastVizSettings = Cast<UDetailPoseModelVizSettings>(DetailPoseModel->GetVizSettings());
			check(CastVizSettings);
			const float DetailPoseWeight = CastVizSettings->GetDetailPoseWeight();
		#else
			const float DetailPoseWeight = 1.0f;
		#endif

		if (DetailPoseModel->GetUseRBFBlending())
		{
			UpdateRBFWeights(OutDetailPoseWeights, DistancePerDetailPose, InOutPrevDetailPoseWeights, DetailPoseModel->GetRBFRange(), ModelWeight * DetailPoseWeight, BlendSpeed);
		}
		else
		{
			UpdateWeights(OutDetailPoseWeights, DistancePerDetailPose, InOutPrevDetailPoseWeights, BlendSpeed, ModelWeight * DetailPoseWeight);
		}
	}
}	// Anonymous namespace


void UDetailPoseModelInstance::Init(USkeletalMeshComponent* SkelMeshComponent)
{
	Super::Init(SkelMeshComponent);

	UDetailPoseModel* DetailPoseModel = Cast<UDetailPoseModel>(Model);
	const int32 NumDetailPoses = DetailPoseModel->GetDetailPoses().Num();

	DetailPoseDistances.Reset();
	DetailPoseDistances.SetNumZeroed(NumDetailPoses);

	DetailPosePrevWeights.Reset();
	DetailPosePrevWeights.SetNumZeroed(NumDetailPoses);
}

void UDetailPoseModelInstance::Execute(float ModelWeight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDetailPoseModelInstance::Execute)
	CSV_SCOPED_TIMING_STAT(MLDeformer, DetailPoseModelInstanceExecute);

	if (!SkeletalMeshComponent)
	{
		return;
	}

	// Grab the weight data for this morph set.
	// This could potentially fail if we are applying this deformer to the wrong skeletal mesh component.
	const int32 LOD = SkeletalMeshComponent->GetPredictedLODLevel();
	FExternalMorphSetWeights* WeightData = FindWeightData(LOD);
	if (!WeightData)
	{
		return;
	}

	// Validate network output expectations.
	UDetailPoseModel* DetailPoseModel = Cast<UDetailPoseModel>(Model);
	UDetailPoseModelInputInfo* InputInfo = Cast<UDetailPoseModelInputInfo>(DetailPoseModel->GetInputInfo());
	const int32 NumGlobalMorphs = InputInfo->GetNumGlobalMorphTargets();
	UNeuralMorphNetwork* Network = DetailPoseModel->GetNeuralMorphNetwork();
	const int32 NumNetworkOutputs = Network ? Network->GetNumMainOutputs() : 0;
	if (!Network || Network->IsEmpty() || NumGlobalMorphs != NumNetworkOutputs || !NetworkInstance)
	{
		WeightData->ZeroWeights();
		return;
	}

	// Calculate the weights the morph targets that the neural network calculates.
	// This excludes the weights of the detail poses. We have concatenated the weights of the detail poses after those weights.
	Super::Execute(ModelWeight);

	// Now that we updated the weights of the morph targets that we generated during training, we have to deal with the
	// weights of our detail poses. We want to calculate those, but we have to do some sanity checks first, and if there is an invalid state.
	// In case there is some issue with the trained model (like if it hasn't been trained), we already zeroed the weights of the 
	// detail poses as well.

	// Now that we know the internal state is valid and we have the weight data, let's calculate and set the weights for the detail poses.
	const int32 NumDetailPoses = WeightData->Weights.Num() - Network->GetNumMainOutputs() - 1;
	if (NumDetailPoses > 0)
	{
		// Calculate the distances from the current pose to each detail pose.
		// We can use these distances to find the closest pose, so we know which detail pose to blend in.
		const TConstArrayView<float> CurrentPoseValues = NetworkInstance->GetInputs();
		CalculateDetailPoseDistances(DetailPoseModel, CurrentPoseValues, DetailPoseDistances);

		#if WITH_EDITOR
			// Find the detail pose index that has the smallest distance (to the current pose).
			const float* MinDistancePtr = Algo::MinElement(DetailPoseDistances);
			BestDetailPoseIndex = MinDistancePtr - DetailPoseDistances.GetData();			  
		#endif

		// Now that we have calculated the distances, update the weights of each detail pose morph target.
		// The weights of the detail poses are concatenated after the regular morph target weights.
		// So we create an array view of just the detail pose weights here.
		const float BlendSpeed = DetailPoseModel->GetBlendSpeed();
		TArrayView<float> DetailPoseWeights(&WeightData->Weights[Network->GetNumMainOutputs() + 1], NumDetailPoses);	// +1 because of the first morph target being the vertex means and isn't part of the network outputs.
		UpdateDetailPoseWeights(DetailPoseModel, DetailPoseWeights, DetailPoseDistances, DetailPosePrevWeights, BlendSpeed, ModelWeight);
	}
}


