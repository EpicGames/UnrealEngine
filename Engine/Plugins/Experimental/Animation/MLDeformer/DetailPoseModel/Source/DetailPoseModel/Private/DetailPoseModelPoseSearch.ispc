// Copyright Epic Games, Inc. All Rights Reserved.

export uniform float CalcDistance(uniform const float A[], uniform const float B[], uniform const int NumItems)
{
	uniform float Sum = 0.0f;
    
    foreach (Index = 0 ... NumItems)
	{
        const varying float Diff = A[Index] - B[Index];
        Sum += reduce_add(Diff * Diff);
    }
    
    return Sum;	
}
