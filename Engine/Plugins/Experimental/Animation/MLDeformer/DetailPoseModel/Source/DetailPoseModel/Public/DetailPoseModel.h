// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "NeuralMorphModel.h"
#include "DetailPoseModel.generated.h"

// Declare our log category.
DETAILPOSEMODEL_API DECLARE_LOG_CATEGORY_EXTERN(LogDetailPoseModel, Log, All);

USTRUCT()
struct DETAILPOSEMODEL_API FDetailPoseModelDetailPose
{
	GENERATED_BODY()

	/** The values that represent the pose. These are the same as the neural network inputs that would represent this pose. */
	UPROPERTY()
	TArray<float> PoseValues;
};

/**
 * The detail pose model for the ML Deformer.
 * This model is inherited from the Neural Morph Model, but adds additional morph targets on top of this.
 * The additional morph targets that are generated are morph targets that bring it to the ground truth as seen during training at specific poses.
 * These special poses are called Detail Poses. We basically look at the deltas that we predict using the neural morph model, and put the remaining error
 * between that and the ground truth into the morph target at the given pose.
 * There can be multiple detail poses, and each of them generates a new morph target.
 */
UCLASS()
class DETAILPOSEMODEL_API UDetailPoseModel 
	: public UNeuralMorphModel
{
	GENERATED_BODY()

public:
	UDetailPoseModel(const FObjectInitializer& ObjectInitializer);

	// UObject overrides.
	void Serialize(FArchive& Archive) override final;
	void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override final;
	// ~END UObject overrides.

	// UMLDeformerModel overrides.
	FString GetDisplayName() const override final						{ return "Detail Pose Model"; }
	UMLDeformerModelInstance* CreateModelInstance(UMLDeformerComponent* Component) override final;
	UMLDeformerInputInfo* CreateInputInfo() override final;
	// ~END UMLDeformerModel overrides.

	// UNeuralMorphModel overrides.
	bool SupportsGlobalModeOnly() const override final					{ return true; }
	// ~END UNeuralMorphModel overrides.

	const TArray<FDetailPoseModelDetailPose>& GetDetailPoses() const	{ return DetailPoses; }
	TArray<FDetailPoseModelDetailPose>& GetDetailPoses()				{ return DetailPoses; }
	float GetBlendSpeed() const											{ return BlendSpeed; }
	bool GetUseRBFBlending() const										{ return bUseRBF; }
	float GetRBFRange() const											{ return RBFRange; }

	void SetUseRBFBlending(bool bUse)									{ bUseRBF = bUse; }
	void SetBlendSpeed(float Speed)										{ check(Speed >= 0.0f && Speed <= 1.0f); BlendSpeed = Speed; }
	void SetRBFRange(float Range)										{ check(Range >= 0.0f); RBFRange = Range; }

#if WITH_EDITORONLY_DATA
	UAnimSequence* GetDetailPosesAnimSequence() const;
	UGeometryCache* GetDetailPosesGeomCache() const;

	static FName GetDetailPosesAnimSequencePropertyName()				{ return GET_MEMBER_NAME_CHECKED(UDetailPoseModel, DetailPosesAnimSequence); }
	static FName GetDetailPosesGeomCachePropertyName()					{ return GET_MEMBER_NAME_CHECKED(UDetailPoseModel, DetailPosesGeomCache); }
	static FName GetBlendSpeedPropertyName()							{ return GET_MEMBER_NAME_CHECKED(UDetailPoseModel, BlendSpeed); }
	static FName GetUseRBFBlendingPropertyName()						{ return GET_MEMBER_NAME_CHECKED(UDetailPoseModel, bUseRBF); }
	static FName GetRBFRangePropertyName()								{ return GET_MEMBER_NAME_CHECKED(UDetailPoseModel, RBFRange); }
#endif

private:
	/** The detail poses. This contains the pose values we use at runtime. */
	UPROPERTY()
	TArray<FDetailPoseModelDetailPose> DetailPoses;

#if WITH_EDITORONLY_DATA
	/**
	 * For certain poses the ML Deformer might not reconstruct important details such as specific cloth folding patterns.
	 * To improve the results near these important poses, you can provide a set of these poses.
	 * The poses are provided similar to the training input data, with an animation sequence and geometry cache pair where each
	 * frame contains a pose you want to preserve details in.
	 * The more poses you add, the higher the memory usage. Each pose will generate an additional morph target.
	 * If you do not add any detail poses, the model will behave the same as the Neural Morph Model in global mode.
	 * Please keep the frame rate and frame count the same for both the anim sequence and geometry cache.
	 */
	UPROPERTY(EditAnywhere, Category = "Detail Poses")
	TSoftObjectPtr<UAnimSequence> DetailPosesAnimSequence;

	/**
	 * For certain poses the ML Deformer might not reconstruct important details such as specific cloth folding patterns.
	 * To improve the results near these important poses, you can provide a set of these poses.
	 * The poses are provided similar to the training input data, with an animation sequence and geometry cache pair where each
	 * frame contains a pose you want to preserve details in.
	 * The more poses you add, the higher the memory usage. Each pose will generate an additional morph target.
	 * If you do not add any detail poses, the model will behave the same as the Neural Morph Model in global mode.
	 * Please keep the frame rate and frame count the same for both the anim sequence and geometry cache.
	 */
	UPROPERTY(EditAnywhere, Category = "Detail Poses")
	TSoftObjectPtr<UGeometryCache> DetailPosesGeomCache;
#endif

	/**
	 * The speed at which the detail poses are blend in. 
	 * Higher values make it blend in faster.
	 * A value of 0.0 would disable the detail poses from being calculated.
	 * A value of 1.0 would disable blending and instantly switch the active detail pose weight.
	 */
	UPROPERTY(EditAnywhere, Category = "Detail Poses", meta = (ClampMin="0.0", ClampMax="1.0"))
	float BlendSpeed = 0.3f;

	/**
	 * Use RBF (Radial Basis Functions) to interpolate?
	 * This will produce higher quality blends between detail poses, at the cost of runtime CPU performance.
	 */
	UPROPERTY(EditAnywhere, DisplayName = "Use RBF Blending", Category = "Detail Poses")
	bool bUseRBF = true;

	/** The range to blend detail poses. Larger values will blend more detail poses together but also result in slower GPU performance. */
	UPROPERTY(EditAnywhere, DisplayName = "RBF Range", Category = "Detail Poses", meta = (ClampMin = "0", EditCondition = "bUseRBF"))
	float RBFRange = 1.0f;
};
