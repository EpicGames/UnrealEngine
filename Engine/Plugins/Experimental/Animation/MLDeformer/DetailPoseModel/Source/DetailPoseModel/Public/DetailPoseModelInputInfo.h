// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "NeuralMorphInputInfo.h"
#include "DetailPoseModelInputInfo.generated.h"

namespace UE::DetailPoseModel
{ 
	class FDetailPoseEditorModel;
}

/**
 * The input info class for the Detail Pose Model.
 * We store values that were used during training. These are often used during inference.
 */
UCLASS()
class DETAILPOSEMODEL_API UDetailPoseModelInputInfo
	: public UNeuralMorphInputInfo
{
	GENERATED_BODY()

public:
	friend class UE::DetailPoseModel::FDetailPoseEditorModel;

	// FMLDeformerInputInfo overrides.
	void Reset() override final;
	// ~END FMLDeformerInputInfo overrides.

	/**
	 * Get the number of morph targets that the model should output.
	 * This is basically the number of outputs of our neural network. Please keep in mind that the number of morph targets
	 * that are actually used is higher than this number. The number returned by this method is really what we entered as 
	 * number of morph targets in the UI.
	 * We do generate an extra morph target that holds the mean values as well, and next to that there is a morph target
	 * for each detail pose. So please keep in mind this is just the number of morph targets we entered in the UI before we pressed 
	 * the Train button, and that it is equal to the number of neural network outputs.
	 */
	int32 GetNumGlobalMorphTargets() const			{ return NumGlobalMorphTargets; }

private:
	void SetNumGlobalMorphTargets(int32 Num)		{ NumGlobalMorphTargets = Num; }

private:
	/** The number of morph targets the training process generated. This excludes the morph target that contains the means. */
	UPROPERTY()
	int32 NumGlobalMorphTargets = 0;
};
