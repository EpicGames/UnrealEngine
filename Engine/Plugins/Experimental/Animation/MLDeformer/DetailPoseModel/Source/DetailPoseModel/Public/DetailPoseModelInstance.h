// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NeuralMorphModelInstance.h"
#include "DetailPoseModelInstance.generated.h"

class USkeletalMeshComponent;

/**
 * The model instance class of the Detail Pose Model.
 * This contains the code that calculates the morph target weights and which calculates
 * which detail pose should be blend in.
 */
UCLASS()
class DETAILPOSEMODEL_API UDetailPoseModelInstance
	: public UNeuralMorphModelInstance
{
	GENERATED_BODY()

public:
	// UMLDeformerModelInstance overrides.
	void Init(USkeletalMeshComponent* SkelMeshComponent) override final;
	void Execute(float ModelWeight) override final;
	// ~END UMLDeformerModelInstance overrides.

	/**
	 * Get the best matching detail pose, compared to the character's current pose as defined by the skeletal mesh component we are linked to.
	 * The integer that is returned represents the frame number inside the detail pose model's detail pose geometry cache.
	 * @return The index of the frame number in the detail pose geometry cache that best matches our current pose.
	 */
	int32 GetBestDetailPoseIndex() const		{ return BestDetailPoseIndex; }

private:
	/** The distance value between the current current pose and each detail pose. */
	TArray<float> DetailPoseDistances;

	/** The detail pose previous frame weights, one for each detail pose. */
	TArray<float> DetailPosePrevWeights;

	/** The detail pose index that currently is the one with closest distance. This is the frame number inside the detail pose geometry cache. */
	int32 BestDetailPoseIndex = -1;
};
