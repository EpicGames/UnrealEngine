// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "NeuralMorphModelVizSettings.h"
#include "DetailPoseModelVizSettings.generated.h"

/**
 * The visualization settings specific to the Detail Pose Model.
 */
UCLASS()
class DETAILPOSEMODEL_API UDetailPoseModelVizSettings
	: public UNeuralMorphModelVizSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	float GetDetailPoseWeight() const						{ return DetailPoseWeight; }
	void SetDetailPoseWeight(float Factor)					{ DetailPoseWeight = FMath::Clamp(Factor, 0.0f, 1.0f); }

	bool GetDrawDetailPose() const							{ return bDrawDetailPose; }
	void SetDrawDetailPose(bool bDraw)						{ bDrawDetailPose = bDraw; }

	static FName GetDetailPoseWeightPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UDetailPoseModelVizSettings, DetailPoseWeight); }
	static FName GetDrawDetailPosePropertyName()			{ return GET_MEMBER_NAME_CHECKED(UDetailPoseModelVizSettings, bDrawDetailPose); }

private:
	/** 
	 * The weight that specifies how much of the detail poses should be added.
	 * When this value is set to 0 it means the detail poses are not visually active.
	 * When set to 1, they are fully active.
	 */
	UPROPERTY(EditAnywhere, Category = "Live Settings|Detail Poses", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float DetailPoseWeight = 1.0f;

	/** Do we want to draw the closest matching detail pose? */
	UPROPERTY(EditAnywhere, Category = "Live Settings|Detail Poses")
	bool bDrawDetailPose = true;
#endif	// WITH_EDITORONLY_DATA
};
