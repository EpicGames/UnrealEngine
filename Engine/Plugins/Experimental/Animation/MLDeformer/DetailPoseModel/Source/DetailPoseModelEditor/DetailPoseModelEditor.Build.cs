// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DetailPoseModelEditor : ModuleRules
	{
		public DetailPoseModelEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"InputCore",
					"UnrealEd",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"Projects",
					"MLDeformerFramework",
					"MLDeformerFrameworkEditor",
					"DetailPoseModel",
					"NeuralMorphModel",
					"NeuralMorphModelEditor",
					"PropertyEditor",
					"ToolWidgets",
					"DeveloperSettings",
					"GeometryCache"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
				}
			);
		}
	}
}
