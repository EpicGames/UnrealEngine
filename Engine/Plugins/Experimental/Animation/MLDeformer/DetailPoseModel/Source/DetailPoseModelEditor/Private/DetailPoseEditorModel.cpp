// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailPoseEditorModel.h"
#include "DetailPoseModel.h"
#include "DetailPoseModelInstance.h"
#include "DetailPoseModelInputInfo.h"
#include "DetailPoseModelEditorStyle.h"
#include "DetailPoseModelEditorActor.h"
#include "DetailPoseModelVizSettings.h"
#include "DetailPoseTrainingModel.h"
#include "MLDeformerComponent.h"
#include "MLDeformerGeomCacheActor.h"
#include "MLDeformerGeomCacheSampler.h"
#include "MLDeformerEditorToolkit.h"
#include "NeuralMorphNetwork.h"
#include "NeuralMorphEditorModel.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Misc/ScopedSlowTask.h"
#include "IPersonaPreviewScene.h"
#include "GeometryCacheComponent.h"
#include "GeometryCache.h"
#include "Async/ParallelFor.h"

#define LOCTEXT_NAMESPACE "DetailPoseEditorModel"

namespace UE::DetailPoseModel
{
	using namespace UE::MLDeformer;

	FMLDeformerEditorModel* FDetailPoseEditorModel::MakeInstance()
	{
		return new FDetailPoseEditorModel();
	}

	FMLDeformerTrainingInputAnim* FDetailPoseEditorModel::GetTrainingInputAnim(int32 Index) const
	{
		if (Index < GetNumTrainingInputAnims())
		{
			return FNeuralMorphEditorModel::GetTrainingInputAnim(Index);
		}
		else
		{
			// Output the detail poses animation.
			check(Index == GetNumTrainingInputAnims());
			if (!DetailPosesAnim.IsValid())
			{
				DetailPosesAnim = MakeUnique<FMLDeformerGeomCacheTrainingInputAnim>();
			}
			DetailPosesAnim->SetAnimSequence(GetDetailPoseModel()->GetDetailPosesAnimSequence());
			DetailPosesAnim->SetGeometryCache(GetDetailPoseModel()->GetDetailPosesGeomCache());
			return DetailPosesAnim.Get();
		}
	}

	void FDetailPoseEditorModel::UpdateTimelineTrainingAnimList()
	{
		TArray<TSharedPtr<FMLDeformerTrainingInputAnimName>> NameList;
		
		// Build the list of names, based on the training inputs.
		const int32 NumAnims = GetNumTrainingInputAnims();
		for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
		{
			const FMLDeformerTrainingInputAnim* Anim = GetTrainingInputAnim(AnimIndex);
			if (Anim)
			{
				if (Anim->IsValid())
				{
					TSharedPtr<FMLDeformerTrainingInputAnimName> AnimName = MakeShared<FMLDeformerTrainingInputAnimName>();
					AnimName->TrainingInputAnimIndex = AnimIndex;
					AnimName->Name = FString::Printf(TEXT("[#%d] %s"), AnimIndex, *Anim->GetAnimSequence()->GetName());
					NameList.Emplace(AnimName);
				}
			}
		}

		const FMLDeformerTrainingInputAnim* Anim = GetTrainingInputAnim(NumAnims);
		if (Anim && Anim->IsValid())
		{
			TSharedPtr<FMLDeformerTrainingInputAnimName> AnimName = MakeShared<FMLDeformerTrainingInputAnimName>();
			AnimName->TrainingInputAnimIndex = NumAnims;
			AnimName->Name = FString::Printf(TEXT("[Detail Poses] %s"), *Anim->GetAnimSequence()->GetName());	
			NameList.Emplace(AnimName);
		}

		SetTimelineAnimNames(NameList);
	}

	void FDetailPoseEditorModel::OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent)
	{
		FNeuralMorphEditorModel::OnPropertyChanged(PropertyChangedEvent);

		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}

		if (Property->GetFName() == UMLDeformerGeomCacheModel::GetTrainingInputAnimsPropertyName() || PropertyChangedEvent.GetMemberPropertyName() == TEXT("TrainingInputAnims"))
		{
			if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove || PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear)
			{
				if (GetDetailPoseModel()->GetTrainingInputAnims().IsEmpty() && DetailPosesAnim.IsValid())
				{
					SetActiveTrainingInputAnimIndex(0);
					TriggerInputAssetChanged();
				}
			}
		}
		else if (Property->GetFName() == UDetailPoseModel::GetDetailPosesAnimSequencePropertyName() || Property->GetFName() == UDetailPoseModel::GetDetailPosesGeomCachePropertyName() || Property->GetFName() == UNeuralMorphModel::GetClampMorphTargetWeightsPropertyName())
		{
			TriggerInputAssetChanged();

			// Empty the detail poses deltas so they get regenerated.
			// And then reinitialize the engine morph targets to include the modified detail pose morphs.
			DetailPosesDeltas.Empty();
			InitEngineMorphTargets(GetMorphModel()->GetMorphTargetDeltas());
		}
		else if (Property->GetFName() == UDetailPoseModelVizSettings::GetDrawDetailPosePropertyName())
		{
			UpdateActorVisibility();
		}
	}

	void FDetailPoseEditorModel::OnPostTraining(ETrainingResult TrainingResult, bool bUsePartiallyTrainedWhenAborted)
	{
		// On successful training, or when we aborted but still want to use the currently trained network,
		// empty the detail poses deltas so they get regenerated.
		// We need to regenerate these detail pose deltas as the trained neural network has changed.
		if (TrainingResult == ETrainingResult::Success || (TrainingResult == ETrainingResult::Aborted && bUsePartiallyTrainedWhenAborted))
		{
			DetailPosesDeltas.Empty();
		}

		FNeuralMorphEditorModel::OnPostTraining(TrainingResult, bUsePartiallyTrainedWhenAborted);
	}

	void FDetailPoseEditorModel::CalculateDetailPoseDeltas(TArray<FVector3f>& OutDeltas, TArray<FDetailPoseModelDetailPose>& OutDetailPoses)
	{
		OutDeltas.Empty();
		OutDetailPoses.Empty();

		const UGeometryCache* DetailPosesGeomCache = GetDetailPoseModel()->GetDetailPosesGeomCache();
		const UAnimSequence* DetailPosesAnimSequence= GetDetailPoseModel()->GetDetailPosesAnimSequence();
		if (!DetailPosesGeomCache || !DetailPosesAnimSequence)
		{
			return;
		}

		const int32 NumDetailPoses = DetailPosesGeomCache->GetEndFrame() - DetailPosesGeomCache->GetStartFrame() + 1;
		if (NumDetailPoses == 0)
		{
			return;
		}

		UDetailPoseModel* DetailPoseModel = GetDetailPoseModel();
		UNeuralMorphNetwork* MorphNetwork = DetailPoseModel->GetNeuralMorphNetwork();
		if (!MorphNetwork)
		{
			return;
		}

		FScopedSlowTask Task(NumDetailPoses, LOCTEXT("CalculateDetailPoseDeltasProgress", "Calculating detail pose morph deltas"));
		Task.MakeDialog(false);	

		// Calculate ground truth deltas.
		FMLDeformerGeomCacheSampler Sampler;
		Sampler.Init(this, GetNumTrainingInputAnims());	// We go one past the number of training input anims here, which will give us the Detail Poses Anim. See GetTrainingInputAnim.
		Sampler.SetVertexDeltaSpace(EVertexDeltaSpace::PreSkinning);
		Sampler.SetSkinningMode(GetDetailPoseModel()->GetSkinningMode());

		// Create and setup the inference object.
		UDetailPoseModelInstance* ModelInstance = NewObject<UDetailPoseModelInstance>();
		ModelInstance->SetModel(Model);
		USkeletalMeshComponent* SkelMeshComponent = NewObject<USkeletalMeshComponent>(ModelInstance);
		SkelMeshComponent->SetSkeletalMesh(Model->GetSkeletalMesh());
		ModelInstance->Init(SkelMeshComponent);
		ModelInstance->PostMLDeformerComponentInit();

		// Get access to the network inputs.
		check(Sampler.GetNumFloatsPerCurve() == 1);
		check(Model->GetNumFloatsPerCurve() == 1);
		const int64 NumDeformerAssetInputs = Model->GetInputInfo()->CalcNumNeuralNetInputs(Model->GetNumFloatsPerBone(), Model->GetNumFloatsPerCurve());
		const int32 NumNeuralNetworkInputs = MorphNetwork->GetNumMainInputs();
		check(NumDeformerAssetInputs == NumNeuralNetworkInputs);

		// Calculate the deltas for each detail pose.
		const int32 NumVerts = Model->GetNumBaseMeshVerts();
		OutDeltas.SetNumZeroed(NumVerts * NumDetailPoses);
		OutDetailPoses.Reset();
		OutDetailPoses.SetNum(NumDetailPoses);
		for (int32 DetailPoseIndex = 0; DetailPoseIndex < NumDetailPoses; ++DetailPoseIndex)
		{
			Sampler.Sample(DetailPoseIndex);

			TArrayView<FVector3f> DetailPoseDeltas(&OutDeltas[DetailPoseIndex * NumVerts], NumVerts);
			CalculateDetailPoseDeltas(DetailPoseIndex, Sampler, ModelInstance, DetailPoseDeltas, OutDetailPoses[DetailPoseIndex]);
			Task.EnterProgressFrame();
		}

		SkelMeshComponent->ConditionalBeginDestroy();
		ModelInstance->ConditionalBeginDestroy();
	}

	void FDetailPoseEditorModel::CalculateDetailPoseDeltas(
		int32 DetailPoseIndex,
		FMLDeformerGeomCacheSampler& Sampler,
		UDetailPoseModelInstance* ModelInstance,
		TArrayView<FVector3f> OutDeltas,
		FDetailPoseModelDetailPose& OutDetailPose)
	{
		// Calculate ground truth deltas.
		const TArray<float>& GroundTruthDeltas = Sampler.GetVertexDeltas();

		// Init the bone inputs.
		UDetailPoseModel* DetailPoseModel = GetDetailPoseModel();
		const TArray<float>& BoneInputFloats = Sampler.GetBoneRotations();
		const TArray<float>& CurveInputFloats= Sampler.GetCurveValues();
		const int32 NumBoneFloats = BoneInputFloats.Num();

		UNeuralMorphNetworkInstance* NetworkInstance = ModelInstance->GetNetworkInstance();
		TArrayView<float> NetworkInputBuffer = NetworkInstance->GetInputs();
		UNeuralMorphNetwork* MorphNetwork = DetailPoseModel->GetNeuralMorphNetwork();
		check(NumBoneFloats + CurveInputFloats.Num() == MorphNetwork->GetNumMainInputs());

		const TConstArrayView<float> Means = MorphNetwork->GetInputMeans();
		const TConstArrayView<float> Stds = MorphNetwork->GetInputStds();
		for (int32 Index = 0; Index < NumBoneFloats; ++Index)
		{
			NetworkInputBuffer[Index] = (BoneInputFloats[Index] - Means[Index]) / Stds[Index];
		}

		// Init the curve inputs.
		for (int32 Index = 0; Index < CurveInputFloats.Num(); ++Index)
		{
			const int32 CurveInputIndex = NumBoneFloats + Index;
			NetworkInputBuffer[CurveInputIndex] = (CurveInputFloats[Index] - Means[CurveInputIndex]) / Stds[CurveInputIndex];
		}

		// Copy the input values as the pose values.
		OutDetailPose.PoseValues = NetworkInputBuffer;

		// Now that we initialized the inputs, run inference.
		NetworkInstance->Run();

		// Clamp the output weights if clamping is enabled.
		TArray<float> MorphWeights(NetworkInstance->GetOutputs());
		if (DetailPoseModel->IsMorphWeightClampingEnabled())
		{
			DetailPoseModel->ClampMorphTargetWeights(MorphWeights);
		}

		// Blend the raw vertex deltas with the morph weights that the neural network has output, to get the predicted deltas.
		const int32 NumVertices = Model->GetNumBaseMeshVerts();
		const TConstArrayView<FVector3f> RawMorphDeltas(DetailPoseModel->GetMorphTargetDeltas().GetData(), NumVertices * (MorphWeights.Num() + 1));
		check(RawMorphDeltas.Num() % NumVertices == 0);
		check(Sampler.GetNumImportedVertices() == NumVertices);
		TArray<FVector3f> PredictedDeltas;
		PredictedDeltas.SetNumZeroed(NumVertices);
		const int32 NumMorphTargets = RawMorphDeltas.Num() / NumVertices;
		check(MorphWeights.Num() + 1 == NumMorphTargets);

		ParallelFor(TEXT("CalcPredictedDetailPoseDeltas"), NumVertices, 500, [&PredictedDeltas, NumMorphTargets, NumVertices, &RawMorphDeltas, &MorphWeights](int32 VertexIndex)
		{
			for (int32 MorphIndex = 0; MorphIndex < NumMorphTargets; ++MorphIndex)
			{
				const int32 MorphVertexOffset = MorphIndex * NumVertices;
				const float MorphWeight = MorphIndex > 0 ? MorphWeights[MorphIndex - 1] : 1.0f;	// The first morph target contains the means that we should add, use a weight of 1 there.
				PredictedDeltas[VertexIndex] += RawMorphDeltas[MorphVertexOffset + VertexIndex] * MorphWeight;
			}
		});

		// Calculate and output the difference between the ground truth and predicted deltas.
		check(PredictedDeltas.Num() * 3 == Sampler.GetVertexDeltas().Num());
		check(PredictedDeltas.Num() == OutDeltas.Num());
		check(OutDeltas.Num() == NumVertices);
		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			const int32 GroundTruthVertexOffset = VertexIndex * 3;
			const FVector3f GroundTruthDelta(GroundTruthDeltas[GroundTruthVertexOffset], GroundTruthDeltas[GroundTruthVertexOffset+1], GroundTruthDeltas[GroundTruthVertexOffset+2]);
			const FVector3f PredictedDelta = PredictedDeltas[VertexIndex];
			OutDeltas[VertexIndex] = GroundTruthDelta - PredictedDelta;	
		}
	}

	FDetailPoseModelEditorActor* FDetailPoseEditorModel::CreateDetailPoseActor(UWorld* World) const
	{
		const FLinearColor LabelColor = FDetailPoseModelEditorStyle::Get().GetColor("DetailPoseModel.EditorActor.LabelColor");
		const FLinearColor WireframeColor = FDetailPoseModelEditorStyle::Get().GetColor("DetailPoseModel.EditorActor.WireframeColor");
		const FName ActorName = FName("DetailPoseActor");
		const FText LabelText = LOCTEXT("DetailPoseActorLabelText", "Detail Pose");
		const int32 ActorID = ActorID_DetailPoseActor;

		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = MakeUniqueObjectName(World, AActor::StaticClass(), ActorName);
		AActor* Actor = World->SpawnActor<AActor>(SpawnParams);
		Actor->SetFlags(RF_Transient);
		
		// Create the Geometry Cache Component.
		UGeometryCacheComponent* GeomCacheComponent = NewObject<UGeometryCacheComponent>(Actor);
		GeomCacheComponent->RegisterComponent();
		GeomCacheComponent->SetOverrideWireframeColor(true);
		GeomCacheComponent->SetWireframeOverrideColor(WireframeColor);
		GeomCacheComponent->MarkRenderStateDirty();
		GeomCacheComponent->SetVisibility(false);
		Actor->SetRootComponent(GeomCacheComponent);
		
		// Create the editor actor.
		UE::MLDeformer::FMLDeformerEditorActor::FConstructSettings Settings;
		Settings.Actor = Actor;
		Settings.TypeID = ActorID;
		Settings.LabelColor = LabelColor;
		Settings.LabelText = LabelText;
		Settings.bIsTrainingActor = false;

		FDetailPoseModelEditorActor* NewActor = new FDetailPoseModelEditorActor(Settings);
		NewActor->SetGeometryCacheComponent(GeomCacheComponent);
		return NewActor;
	}

	UMLDeformerComponent* FDetailPoseEditorModel::GetTestMLDeformerComponent() const
	{
		return FindMLDeformerComponent(UE::MLDeformer::ActorID_Test_MLDeformed);
	}

	void FDetailPoseEditorModel::UpdateDetailPoseActor(FDetailPoseModelEditorActor& Actor) const
	{
		using namespace UE::MLDeformer;

		const UDetailPoseModel* DetailPoseModel = GetDetailPoseModel();
		if (!DetailPoseModel)
		{
			return;
		}

		float MaxOffset = 0.0f;
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && 
				EditorActor->GetTypeID() != ActorID_DetailPoseActor &&
				EditorActor->IsVisible()) 
			{
				MaxOffset = FMath::Max(EditorActor->GetMeshOffsetFactor(), MaxOffset);
			}
		}

		Actor.SetMeshOffsetFactor(MaxOffset + 1.0f);

		UGeometryCache* GeomCache = DetailPoseModel->GetDetailPosesGeomCache();
		Actor.SetGeometryCache(GeomCache);

		const UMLDeformerComponent* MLDeformerComponent = GetTestMLDeformerComponent();
		Actor.SetTrackedComponent(MLDeformerComponent);
	}

	void FDetailPoseEditorModel::InitInputInfo(UMLDeformerInputInfo* InputInfo)
	{
		FNeuralMorphEditorModel::InitInputInfo(InputInfo);

		UDetailPoseModelInputInfo* CastInputInfo = Cast<UDetailPoseModelInputInfo>(InputInfo);
		if (!CastInputInfo)
		{
			return;
		}

		CastInputInfo->SetNumGlobalMorphTargets(GetDetailPoseModel()->GetGlobalNumMorphs());
	}

	void FDetailPoseEditorModel::CreateActors(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
	{
		FNeuralMorphEditorModel::CreateActors(InPersonaPreviewScene);

		// Create the detail pose actor.
		UWorld* const World = InPersonaPreviewScene->GetWorld();
		if (!World)
		{
			return;
		}

		if (DetailPoseActor)
		{
			EditorActors.Remove(DetailPoseActor);
			if (DetailPoseActor->GetActor())
			{
				World->DestroyActor(DetailPoseActor->GetActor(), true);
			}
		}
		DetailPoseActor = CreateDetailPoseActor(World);
		UpdateDetailPoseActor(*DetailPoseActor);

		EditorActors.Add(DetailPoseActor);
	}

	void FDetailPoseEditorModel::InitEngineMorphTargets(const TArray<FVector3f>& Deltas)
	{
		TArray<FVector3f> FinalDeltas = Deltas;

		// Add the deltas for all the detail pose morph targets.
		// We can only do this when we actually trained the model.
		// Also skip calculating them if we already have them calculated, unless there is some vertex count mismatch.
		if (IsTrained() && (DetailPosesDeltas.IsEmpty() || DetailPosesDeltas.Num() % Model->GetNumBaseMeshVerts() != 0))
		{
			TArray<FDetailPoseModelDetailPose>& DetailPoses = GetDetailPoseModel()->GetDetailPoses();
			CalculateDetailPoseDeltas(DetailPosesDeltas, DetailPoses);
		}

		// Strip the existing detail pose deltas if we have a trained network.
		const UNeuralMorphNetwork* MorphNetwork = GetDetailPoseModel()->GetNeuralMorphNetwork();
		if (MorphNetwork)
		{
			const int32 NumTrainedMorphTargets = MorphNetwork->GetNumOutputs() + 1;	// Add one to add the means morph target.
			const TConstArrayView<FVector3f> MainMorphs(Deltas.GetData(), NumTrainedMorphTargets * Model->GetInputInfo()->GetNumBaseMeshVertices());
			FinalDeltas = MainMorphs;
		}
		FinalDeltas += DetailPosesDeltas;
		GetMorphModel()->SetMorphTargetDeltas(FinalDeltas);

		FNeuralMorphEditorModel::InitEngineMorphTargets(FinalDeltas);
	}

	void FDetailPoseEditorModel::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
	{
		FMLDeformerEditorModel::Tick(ViewportClient, DeltaTime);
		if (DetailPoseActor)
		{
			UpdateDetailPoseActor(*DetailPoseActor);
			DetailPoseActor->Tick();
		}
	}

	void FDetailPoseEditorModel::UpdateActorVisibility()
	{
		FNeuralMorphEditorModel::UpdateActorVisibility();

		if (!DetailPoseActor)
		{
			return;
		}

		const UDetailPoseModelVizSettings* VizSettings = Cast<UDetailPoseModelVizSettings>(Model->GetVizSettings());
		const bool bShowTestData = (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData);
		const bool bInDefaultMode = GetEditor()->IsDefaultModeActive();
		const bool bIsVisible = bShowTestData && VizSettings->GetDrawDetailPose() && DetailPoseActor->HasVisualMesh() && bInDefaultMode;

		DetailPoseActor->SetVisibility(bIsVisible);
	}

	ETrainingResult FDetailPoseEditorModel::Train()
	{
		return TrainModel<UDetailPoseTrainingModel>(this);
	}
}	// namespace UE::DetailPoseModel

#undef LOCTEXT_NAMESPACE
