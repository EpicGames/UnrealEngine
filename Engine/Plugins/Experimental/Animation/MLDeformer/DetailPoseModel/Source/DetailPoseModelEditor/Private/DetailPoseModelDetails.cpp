// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailPoseModelDetails.h"
#include "DetailPoseEditorModel.h"
#include "NeuralMorphModelDetails.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Layout/SBox.h"

namespace UE::DetailPoseModel
{
	TSharedRef<IDetailCustomization> FDetailPoseModelDetails::MakeInstance()
	{
		return MakeShareable(new FDetailPoseModelDetails());
	}

	void FDetailPoseModelDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		using namespace UE::MLDeformer;

		DetailLayoutBuilder = &DetailBuilder;

		// Update the pointers and check if they are valid.
		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);
		if (!UpdateMemberPointers(Objects))
		{
			return;
		}

		CreateCategories();

		UDetailPoseModel* DetailPoseModel = GetCastModel();
		check(DetailPoseModel);

		// Check if the geom cache is compatible with the skeletal mesh.
		const FText GeomCacheErrorText = GetGeomCacheErrorText(Model->GetSkeletalMesh(), DetailPoseModel->GetDetailPosesGeomCache());
		FDetailWidgetRow& GeomCacheErrorRow = DetailPosesCategoryBuilder->AddCustomRow(FText::FromString("DetailPosesError"))
			.Visibility(!GeomCacheErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Error)
					.Message(GeomCacheErrorText)
				]
			];

		// Check the animation frame rate and frame count etc.
		const FText GeomCacheAnimErrorText = GetGeomCacheAnimSequenceErrorText(DetailPoseModel->GetDetailPosesGeomCache(), DetailPoseModel->GetDetailPosesAnimSequence());
		FDetailWidgetRow& GeomCacheAnimErrorRow = DetailPosesCategoryBuilder->AddCustomRow(FText::FromString("DetailPosesAnimError"))
			.Visibility(!GeomCacheAnimErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Error)
					.Message(GeomCacheAnimErrorText)
				]
			];

		// Check the mesh mappings between the geom cache and skeletal mesh.
		const FText GeomCacheMappingErrorText = GetGeomCacheMeshMappingErrorText(DetailPoseModel->GetSkeletalMesh(), DetailPoseModel->GetDetailPosesGeomCache());
		FDetailWidgetRow& GeomCacheMappingErrorRow = DetailPosesCategoryBuilder->AddCustomRow(FText::FromString("DetailPosesMappingError"))
			.Visibility(!GeomCacheMappingErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Error)
					.Message(GeomCacheMappingErrorText)
				]
			];
	}

	void FDetailPoseModelDetails::CreateCategories()
	{
		FNeuralMorphModelDetails::CreateCategories();
		DetailPosesCategoryBuilder = &DetailLayoutBuilder->EditCategory("Detail Poses", FText::GetEmpty(), ECategoryPriority::Default);
	}

	UDetailPoseModel* FDetailPoseModelDetails::GetCastModel() const
	{
		return Cast<UDetailPoseModel>(Model);
	}
}	// namespace UE::DetailPoseModel
