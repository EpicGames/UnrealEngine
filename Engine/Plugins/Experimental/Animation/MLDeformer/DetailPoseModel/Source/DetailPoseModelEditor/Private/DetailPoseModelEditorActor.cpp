// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailPoseModelEditorActor.h"
#include "DetailPoseModelInstance.h"
#include "MLDeformerComponent.h"
#include "GeometryCacheComponent.h"
#include "GeometryCache.h"

namespace UE::DetailPoseModel
{
	FDetailPoseModelEditorActor::FDetailPoseModelEditorActor(const FConstructSettings& Settings)
		: FMLDeformerGeomCacheActor(Settings)
	{
	}

	FDetailPoseModelEditorActor::~FDetailPoseModelEditorActor() = default;

	void FDetailPoseModelEditorActor::SetGeometryCache(UGeometryCache* InGeometryCache) const
	{
		if (GeomCacheComponent && GeomCacheComponent->GetGeometryCache() != InGeometryCache)
		{
			GeomCacheComponent->SetGeometryCache(InGeometryCache);
		}
	}

	void FDetailPoseModelEditorActor::SetTrackedComponent(const UMLDeformerComponent* InComponent)
	{
		TrackedComponent = InComponent;
	}

	void FDetailPoseModelEditorActor::Tick() const
	{
		if (!GeomCacheComponent || !GeomCacheComponent->GetGeometryCache() || !TrackedComponent.IsValid())
		{
			return;
		}

		const UDetailPoseModelInstance* ModelInstance = Cast<UDetailPoseModelInstance>(TrackedComponent->GetModelInstance());
		if (ModelInstance)
		{
			const UGeometryCache* GeomCache = GeomCacheComponent->GetGeometryCache();
			const int32 FrameIndex = ModelInstance->GetBestDetailPoseIndex();
			if (FrameIndex >= 0 && FrameIndex <= (GeomCache->GetEndFrame() - GeomCache->GetStartFrame() + 1))
			{
				GeomCacheComponent->SetManualTick(true);
				const float NewTimeValue = GeomCacheComponent->GetTimeAtFrame(FrameIndex);
				const float CurrentTime = GeomCacheComponent->GetAnimationTime();
				if (FMath::Abs(NewTimeValue - CurrentTime) > 0.00001f)
				{
					GeomCacheComponent->TickAtThisTime(NewTimeValue, false, false, false);
				}
			}
		}
	}
}	// namespace UE::DetailPoseModel
