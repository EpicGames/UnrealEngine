// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailPoseEditorModel.h"
#include "DetailPoseModel.h"
#include "DetailPoseModelDetails.h"
#include "MLDeformerEditorModule.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "DetailPoseModelEditorModule"

namespace UE::DetailPoseModel
{
	class FDetailPoseModelEditorModule
		: public IModuleInterface
	{
	public:
		// IModuleInterface overrides.
		void StartupModule() override;
		void ShutdownModule() override;
		// ~END IModuleInterface overrides.
	};
}
IMPLEMENT_MODULE(UE::DetailPoseModel::FDetailPoseModelEditorModule, DetailPoseModelEditor)

namespace UE::DetailPoseModel
{
	using namespace UE::MLDeformer;

	void FDetailPoseModelEditorModule::StartupModule()
	{
		// Register object detail customizations.
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("DetailPoseModel", FOnGetDetailCustomizationInstance::CreateStatic(&FDetailPoseModelDetails::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();

		// Register our custom ML deformer model to the model registry in the ML Deformer Framework.
		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();
		ModelRegistry.RegisterEditorModel(UDetailPoseModel::StaticClass(), FOnGetEditorModelInstance::CreateStatic(&FDetailPoseEditorModel::MakeInstance), /*ModelPriority*/10);
	}

	void FDetailPoseModelEditorModule::ShutdownModule()
	{
		// Unregister our ML Deformer model.
		if (FModuleManager::Get().IsModuleLoaded(TEXT("MLDeformerFrameworkEditor")))
		{
			FMLDeformerEditorModule& EditorModule = FModuleManager::GetModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
			FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();
			ModelRegistry.UnregisterEditorModel(UDetailPoseModel::StaticClass());
		}

		// Unregister object detail customizations for this model.
		if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
			PropertyModule.UnregisterCustomClassLayout(TEXT("DetailPoseModel"));
			PropertyModule.NotifyCustomizationModuleChanged();
		}
	}
}	// namespace UE::DetailPoseModel

#undef LOCTEXT_NAMESPACE
