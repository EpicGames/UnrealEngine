// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailPoseModelEditorStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Math/Color.h"
#include "Misc/Paths.h"

namespace UE::DetailPoseModel
{
	FDetailPoseModelEditorStyle::FDetailPoseModelEditorStyle()
		: FSlateStyleSet("DetailPoseModelEditorStyle")
	{
		const FString ResourceDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("DetailPoseModel"))->GetBaseDir(), TEXT("Resources"));
		SetContentRoot(ResourceDir);

		// Colors and sizes.
		Set("DetailPoseModel.EditorActor.WireframeColor", FLinearColor(0.0f, 1.0f, 1.0f));
		Set("DetailPoseModel.EditorActor.LabelColor", FLinearColor(0.0f, 1.0f, 1.0f));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FDetailPoseModelEditorStyle::~FDetailPoseModelEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	FDetailPoseModelEditorStyle& FDetailPoseModelEditorStyle::Get()
	{
		static FDetailPoseModelEditorStyle Inst;
		return Inst;
	}
}	// namespace UE::DetailPoseModel

#undef MLDEFORMER_IMAGE_BRUSH
