// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "NeuralMorphEditorModel.h"
#include "DetailPoseModel.h"
#include "Templates/UniquePtr.h"

class UDetailPoseModelInstance;
class UNeuralMorphNetworkInstance;

namespace UE::DetailPoseModel
{
	using namespace UE::MLDeformer;

	class FDetailPoseModelEditorActor;

	/**
	 * The editor model for the Detail Pose Model.
	 */
	class DETAILPOSEMODELEDITOR_API FDetailPoseEditorModel
		: public UE::NeuralMorphModel::FNeuralMorphEditorModel
	{
	public:
		// We need to implement this static MakeInstance method.
		static FMLDeformerEditorModel* MakeInstance();

		// FGCObject overrides.
		FString GetReferencerName() const override final						{ return TEXT("FDetailPoseEditorModel"); }
		// ~END FGCObject overrides.
	
		// FMLDeformerEditorModel overrides.
		FMLDeformerTrainingInputAnim* GetTrainingInputAnim(int32 Index) const override final;
		void UpdateTimelineTrainingAnimList() override final;
		void OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent) override final;
		void OnPostTraining(ETrainingResult TrainingResult, bool bUsePartiallyTrainedWhenAborted) override final;
		void InitInputInfo(UMLDeformerInputInfo* InputInfo) override final;
		void CreateActors(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene) override final;
		void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override final;
		void UpdateActorVisibility() override final;
		ETrainingResult Train() override final;
		// ~END FMLDeformerEditorModel overrides.

		// FMLDeformerMorphModelEditorModel overrides.
		void InitEngineMorphTargets(const TArray<FVector3f>& Deltas) override final;
		// ~END FMLDeformerMorphModelEditorModel overrides.

		/** Get a pointer to the runtime model. */
		UDetailPoseModel* GetDetailPoseModel() const							{ return Cast<UDetailPoseModel>(Model); }

	private:
		void CalculateDetailPoseDeltas(TArray<FVector3f>& OutDeltas, TArray<FDetailPoseModelDetailPose>& OutDetailPoses);
		void CalculateDetailPoseDeltas(int32 DetailPoseIndex, FMLDeformerGeomCacheSampler& Sampler, UDetailPoseModelInstance* ModelInstance, TArrayView<FVector3f> OutDeltas, FDetailPoseModelDetailPose& OutDetailPose);

		FDetailPoseModelEditorActor* CreateDetailPoseActor(UWorld* World) const;
		void UpdateDetailPoseActor(FDetailPoseModelEditorActor& Actor) const;

		UMLDeformerComponent* GetTestMLDeformerComponent() const;

	private:
		/** The training input anim object that holds the detail pose anim sequence and geom cache sequence. */
		mutable TUniquePtr<FMLDeformerGeomCacheTrainingInputAnim> DetailPosesAnim;

		/** The detail pose deltas, of size (NumDetailPoses * NumBaseMeshVerts). */
		TArray<FVector3f> DetailPosesDeltas;

		/** 
		 * The editor actor that represents the detail pose actor. It is used to show the currently used detail pose. 
		 * It's creation and destruction are managed internally.
		 * @see CreateActors.
		 */
		FDetailPoseModelEditorActor* DetailPoseActor = nullptr;
	};

}	// namespace UE::DetailPoseModel
