// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "NeuralMorphModelDetails.h"

class IDetailChildrenBuilder;
class IPropertyHandle;
class UDetailPoseModel;

namespace UE::DetailPoseModel
{
	class FDetailPoseEditorModel;

	/**
	 * The details customization for the model settings of the Detail Pose Model.
	 * We implement a detail customization because we want to show some errors/warnings in case there are
	 * any issues with the detail pose animation sequence or geometry cache.
	 * Those errors could be like mismatching frame numbers etc.
	 */
	class DETAILPOSEMODELEDITOR_API FDetailPoseModelDetails
		: public UE::NeuralMorphModel::FNeuralMorphModelDetails
	{
	public:
		/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
		static TSharedRef<IDetailCustomization> MakeInstance();

		// ILayoutDetails overrides.
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override final;
		virtual void CreateCategories() override final;
		// ~END ILayoutDetails overrides.

	private:
		/**
		 * Get a cast version of the UMLDeformerModel that we are showing details for.
		 * This casts the model to the UDetailPoseModel class.
		 * @result A pointer to the UDetailPoseModel that we are currently customizing, or nullptr when something is wrong.
		 */
		UDetailPoseModel* GetCastModel() const;

	private:
		/** The category that holds the Detail Pose settings in the model details panel. */
		IDetailCategoryBuilder* DetailPosesCategoryBuilder = nullptr;
	};
}	// namespace UE::DetailPoseModel
