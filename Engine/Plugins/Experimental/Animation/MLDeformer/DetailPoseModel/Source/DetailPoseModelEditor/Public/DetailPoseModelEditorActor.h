// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerGeomCacheActor.h"

class UGeometryCache;
class UMLDeformerComponent;

namespace UE::DetailPoseModel
{
	enum : int32
	{
		/** The detail pose actor, which shows the current closest matching detail pose. */
		ActorID_DetailPoseActor = 6
	};

	/**
	 * The editor actor for the Detail Pose model.
	 * This represents the visual detail pose actor inside the ML Deformer asset editor's viewport.
	 * It is used to show the current closest matching detail pose, compared to the current pose of our skeletal mesh component.
	 */
	class DETAILPOSEMODELEDITOR_API FDetailPoseModelEditorActor
		: public UE::MLDeformer::FMLDeformerGeomCacheActor
	{
	public:
		FDetailPoseModelEditorActor(const FConstructSettings& Settings);
		virtual ~FDetailPoseModelEditorActor();

		/**
		 * Set the geometry cache asset to use to render in the editor viewport.
		 * This should be the detail pose geometry cache.
		 * @param InGeometryCache The geometry cache to use to render the detail poses.
		 */
		void SetGeometryCache(UGeometryCache* InGeometryCache) const;

		/**
		 * Specify which ML Deformer component to extract the current closest detail pose from.
		 * The Tick method will use this and update the geometry cache components current time value based on this.
		 * @param InComponent The ML Deformer Component to grab the current frame index from inside the detail pose geometry cache.
		 * @see Tick.
		 */
		void SetTrackedComponent(const UMLDeformerComponent* InComponent);

		/**
		 * Tick the editor actor, which will basically tick the geometry cache component and set it to the
		 * frame that is the current closest detail pose as reported by the tracked ML Deformer component.
		 * The ML Deformer component (or well the UDetailPoseModelInstance inside it) stores the current closest detail pose.
		 * We grab the frame number of that pose (the frame number inside the Geometry Cache), and then set the internal
		 * geometry cache component's time to this frame number to display this pose.
		 */
		void Tick() const;
	
	private:
		/** The ML Deformer component from which we grab the current frame value. */
		TWeakObjectPtr<const UMLDeformerComponent> TrackedComponent;
	};
}	// namespace UE::DetailPoseModel
