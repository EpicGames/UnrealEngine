// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::DetailPoseModel
{
	/**
	 * The styles used in the Detail Pose editor model.
	 * This contains things like colors for the label of the Detail Pose model rendered in the viewport.
	 */
	class DETAILPOSEMODELEDITOR_API FDetailPoseModelEditorStyle
		: public FSlateStyleSet
	{
	public:
		FDetailPoseModelEditorStyle();
		virtual ~FDetailPoseModelEditorStyle();

		static FDetailPoseModelEditorStyle& Get();
	};
}	// namespace UE::DetailPoseModel
