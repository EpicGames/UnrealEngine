// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NeuralMorphTrainingModel.h"
#include "DetailPoseTrainingModel.generated.h"

/**
 * The Training Model for the Detail Pose Model.
 * Even though this is an empty class we still need it, as we inherit from it inside the 
 * Python code. This allows us to separate it from the Neural Morph Model that we inherited from.
 * We basically find an object of this class type and call the Train method on that in Python.
 */
UCLASS(Blueprintable)
class DETAILPOSEMODELEDITOR_API UDetailPoseTrainingModel
	: public UNeuralMorphTrainingModel
{
	GENERATED_BODY()
};
